.sarif/
.vs/
vcpkg_installed/
build/
.xmake/
vsxmake*/
