This code corresponds to the code in [UE5/Engine/Source/Developer/ZenPluggableTransport](https://github.com/EpicGames/UnrealEngine/tree/release/Engine/Source/Developer), 
and provides the API definitions for creating transport plug-ins for use with the
Zen server. Pluggable transports allow us to support a variety of transport mechanisms, 
including some which are not possible to share with a general audience. These are 
developed and maintained in the UE tree or elsewhere for logistical and legal reasons.
