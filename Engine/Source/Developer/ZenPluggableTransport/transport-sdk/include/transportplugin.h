// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdint.h>

// This header is meant to compile standalone and should therefore NOT depend
// on anything from the Zen tree

//////////////////////////////////////////////////////////////////////////
//
// IMPORTANT: Any development or changes to this header should be made
// in the Zen repository http://github.com/epicgames/zen even if you
// may find the header in the UE tree
//
//////////////////////////////////////////////////////////////////////////

namespace zen {

// Current API version, value will be incremented to represent breaking changes
static const uint32_t kTransportApiVersion = 1;

class TransportConnection;
class TransportPlugin;
class TransportServerConnection;
class TransportServer;
class TransportLogger;

/*************************************************************************

   The following interfaces are implemented on the server side, and instances
   are provided to the plugins.

*************************************************************************/

/** Plugin-server interface for connection

	This is returned by a call to TransportServer::CreateConnectionHandler
	and there should be one instance created per established connection

	The plugin uses this interface to feed data into the server side
	protocol implementation which will parse the incoming messages and
	dispatch to appropriate request handlers and ultimately call into
	TransportConnection functions which write data back to the client
 */
class TransportServerConnection
{
public:
	virtual uint32_t AddRef() const									  = 0;
	virtual uint32_t Release() const								  = 0;
	virtual void	 OnBytesRead(const void* Buffer, size_t DataSize) = 0;
};

/** Server interface

 There will be one instance of this provided by the system to the transport plugin

 The plugin can use this to register new connections

 */
class TransportServer
{
public:
	virtual TransportServerConnection* CreateConnectionHandler(TransportConnection* Connection) = 0;
};

/** Logger interface

 There will be one instance of this provided by the system to the transport plugin

 The plugin can use this to log messages back to zen server

 */
class TransportLogger
{
public:
	enum class LogLevel : uint32_t
	{
		Trace	 = 0,
		Debug	 = 1,
		Info	 = 2,
		Warn	 = 3,
		Err		 = 4,
		Critical = 5,
	};

	virtual void LogMessage(LogLevel Level, const char* Message) = 0;
};

/*************************************************************************

   The following interfaces are to be implemented by transport plugins.

*************************************************************************/

/** Interface which needs to be implemented by a transport plugin

	This is responsible for setting up and running the communication
	for a given transport.

	Once initialized, the plugin should be ready to accept connections
	using its own execution resources (threads, thread pools etc)
 */
class TransportPlugin
{
public:
	virtual uint32_t	AddRef() const											  = 0;
	virtual uint32_t	Release() const											  = 0;
	virtual void		Configure(const char* OptionTag, const char* OptionValue) = 0;
	virtual void		Initialize(TransportServer* ServerInterface)			  = 0;
	virtual void		Shutdown()												  = 0;
	virtual const char* GetDebugName()											  = 0;

	/** Check whether this transport is usable.
	 */
	virtual bool IsAvailable() = 0;
};

/** A transport plugin provider needs to implement this interface

   The plugin should create one instance of this per established
   connection and register it with the TransportServer instance
   CreateConnectionHandler() function. The server will subsequently
   use this interface to write response data back to the client and
   to manage the connection life cycle in general
*/
class TransportConnection
{
public:
	virtual int64_t		WriteBytes(const void* Buffer, size_t DataSize) = 0;
	virtual void		Shutdown(bool Receive, bool Transmit)			= 0;
	virtual void		CloseConnection()								= 0;
	virtual const char* GetDebugName()									= 0;
};

}  // namespace zen

#if defined(_MSC_VER)
#	define DLL_TRANSPORT_API __declspec(dllexport)
#else
#	define DLL_TRANSPORT_API
#endif

extern "C"
{
	/** Provide information about plugin version

	   Fills out API version (kTransportApiVersion) plugin was built against.
	   Fills out plugin own version ever increasing version number,
	   a copy of plugin with higher version will be used.
	*/
	DLL_TRANSPORT_API void GetTransportPluginVersion(uint32_t* OutApiVersion, uint32_t* OutPluginVersion);

	// Return nullptr if requested api version mismatches api version plugin was built against
	DLL_TRANSPORT_API zen::TransportPlugin* CreateTransportPlugin(zen::TransportLogger* Logger);
}

typedef void (*PfnGetTransportPluginVersion)(uint32_t* OutApiVersion, uint32_t* OutPluginVersion);

typedef zen::TransportPlugin* (*PfnCreateTransportPlugin)(zen::TransportLogger* Logger);
