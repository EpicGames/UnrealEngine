-- Copyright Epic Games, Inc. All Rights Reserved.

target('transport-sdk')
    set_kind("headeronly")
    set_group("transports")
    add_headerfiles("**.h")
    add_includedirs("include", {public=true})
