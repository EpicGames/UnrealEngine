// Copyright Epic Games, Inc. All Rights Reserved.

#include <inttypes.h>
#include <atomic>
#include <charconv>
#include <exception>
#include <future>
#include <memory>
#include <optional>
#include <thread>

#include <zenbase/concepts.h>
#include <zenbase/refcount.h>

#ifndef _WIN32_WINNT
#	define _WIN32_WINNT 0x0A00
#endif

ZEN_THIRD_PARTY_INCLUDES_START
#include <WS2tcpip.h>
#include <WinSock2.h>
#include <windows.h>
ZEN_THIRD_PARTY_INCLUDES_END

#include <transportplugin.h>

//////////////////////////////////////////////////////////////////////////

template<Integral T>
std::optional<T>
ParseInt(const std::string_view& Input)
{
	T							 Out	= 0;
	const std::from_chars_result Result = std::from_chars(Input.data(), Input.data() + Input.size(), Out);
	if (Result.ec == std::errc::invalid_argument || Result.ec == std::errc::result_out_of_range)
	{
		return std::nullopt;
	}
	return Out;
}

//////////////////////////////////////////////////////////////////////////

using namespace zen;

class WinsockTransportPlugin : public TransportPlugin, zen::RefCounted
{
public:
	WinsockTransportPlugin();
	~WinsockTransportPlugin();

	// TransportPlugin implementation

	virtual uint32_t	AddRef() const override;
	virtual uint32_t	Release() const override;
	virtual void		Configure(const char* OptionTag, const char* OptionValue) override;
	virtual void		Initialize(TransportServer* ServerInterface) override;
	virtual void		Shutdown() override;
	virtual const char* GetDebugName() override;
	virtual bool		IsAvailable() override;

private:
	TransportServer* m_ServerInterface = nullptr;
	bool			 m_IsOk			   = true;
	uint16_t		 m_BasePort		   = 0;

	SOCKET						   m_ListenSocket{};
	std::thread					   m_AcceptThread;
	std::atomic_flag			   m_KeepRunning;
	std::vector<std::future<void>> m_Connections;
};

struct WinsockTransportConnection : public TransportConnection
{
public:
	WinsockTransportConnection();
	~WinsockTransportConnection();

	void Initialize(TransportServerConnection* ServerConnection, SOCKET ClientSocket);
	void HandleConnection();

	// TransportConnection implementation

	virtual int64_t		WriteBytes(const void* Buffer, size_t DataSize) override;
	virtual void		Shutdown(bool Receive, bool Transmit) override;
	virtual void		CloseConnection() override;
	virtual const char* GetDebugName() override;

private:
	zen::Ref<TransportServerConnection> m_ConnectionHandler;
	SOCKET								m_ClientSocket{};
	bool								m_IsTerminated = false;
};

//////////////////////////////////////////////////////////////////////////

WinsockTransportConnection::WinsockTransportConnection()
{
}

WinsockTransportConnection::~WinsockTransportConnection()
{
}

void
WinsockTransportConnection::Initialize(TransportServerConnection* ServerConnection, SOCKET ClientSocket)
{
	// ZEN_ASSERT(!m_ConnectionHandler);

	m_ConnectionHandler = ServerConnection;
	m_ClientSocket		= ClientSocket;
}

void
WinsockTransportConnection::HandleConnection()
{
	// ZEN_ASSERT(m_ConnectionHandler);

	const int				   InputBufferSize = 64 * 1024;
	std::unique_ptr<uint8_t[]> InputBuffer{new uint8_t[64 * 1024]};

	do
	{
		const int RecvBytes = recv(m_ClientSocket, (char*)InputBuffer.get(), InputBufferSize, /* flags */ 0);

		if (RecvBytes == 0)
		{
			// Connection closed
			return CloseConnection();
		}
		else if (RecvBytes < 0)
		{
			// Error
			return CloseConnection();
		}

		m_ConnectionHandler->OnBytesRead(InputBuffer.get(), RecvBytes);
	} while (m_ClientSocket);
}

void
WinsockTransportConnection::CloseConnection()
{
	if (m_IsTerminated)
	{
		return;
	}

	// ZEN_ASSERT(m_ClientSocket);
	m_IsTerminated = true;

	shutdown(m_ClientSocket, SD_BOTH);	// We won't be sending or receiving any more data

	closesocket(m_ClientSocket);
	m_ClientSocket = 0;
}

const char*
WinsockTransportConnection::GetDebugName()
{
	return nullptr;
}

int64_t
WinsockTransportConnection::WriteBytes(const void* Buffer, size_t DataSize)
{
	const uint8_t* BufferCursor	  = reinterpret_cast<const uint8_t*>(Buffer);
	int64_t		   TotalBytesSent = 0;

	while (DataSize)
	{
		const int MaxBlockSize	= 128 * 1024;
		const int SendBlockSize = (DataSize > MaxBlockSize) ? MaxBlockSize : (int)DataSize;
		const int SentBytes		= send(m_ClientSocket, (const char*)BufferCursor, SendBlockSize, /* flags */ 0);

		if (SentBytes < 0)
		{
			// Error
			return SentBytes;
		}

		BufferCursor += SentBytes;
		DataSize -= SentBytes;
		TotalBytesSent += SentBytes;
	}

	return TotalBytesSent;
}

void
WinsockTransportConnection::Shutdown(bool Receive, bool Transmit)
{
	if (Receive)
	{
		if (Transmit)
		{
			shutdown(m_ClientSocket, SD_BOTH);
		}
		else
		{
			shutdown(m_ClientSocket, SD_RECEIVE);
		}
	}
	else if (Transmit)
	{
		shutdown(m_ClientSocket, SD_SEND);
	}
}

//////////////////////////////////////////////////////////////////////////

WinsockTransportPlugin::WinsockTransportPlugin()
{
#if ZEN_PLATFORM_WINDOWS
	WSADATA wsaData;
	if (int Result = WSAStartup(0x202, &wsaData); Result != 0)
	{
		m_IsOk = false;
		WSACleanup();
	}
#endif
}

WinsockTransportPlugin::~WinsockTransportPlugin()
{
	Shutdown();

#if ZEN_PLATFORM_WINDOWS
	if (m_IsOk)
	{
		WSACleanup();
	}
#endif
}

uint32_t
WinsockTransportPlugin::AddRef() const
{
	return RefCounted::AddRef();
}

uint32_t
WinsockTransportPlugin::Release() const
{
	return RefCounted::Release();
}

void
WinsockTransportPlugin::Configure(const char* OptionTag, const char* OptionValue)
{
	using namespace std::literals;

	if (OptionTag == "port"sv)
	{
		if (auto PortNum = ParseInt<uint16_t>(OptionValue))
		{
			m_BasePort = *PortNum;
		}
	}
	else
	{
		// Unknown configuration option
	}
}

void
WinsockTransportPlugin::Initialize(TransportServer* ServerInterface)
{
	uint16_t Port = m_BasePort;

	m_ServerInterface = ServerInterface;
	m_ListenSocket	  = socket(AF_INET6, SOCK_STREAM, 0);

	if (m_ListenSocket == SOCKET_ERROR || m_ListenSocket == INVALID_SOCKET)
	{
		throw std::system_error(std::error_code(WSAGetLastError(), std::system_category()),
								"socket creation failed in HTTP plugin server init");
	}

	sockaddr_in6 Server{};
	Server.sin6_family = AF_INET6;
	Server.sin6_port   = htons(Port);
	Server.sin6_addr   = in6addr_any;

	if (int Result = bind(m_ListenSocket, (sockaddr*)&Server, sizeof(Server)); Result == SOCKET_ERROR)
	{
		throw std::system_error(std::error_code(WSAGetLastError(), std::system_category()), "bind call failed in HTTP plugin server init");
	}

	if (int Result = listen(m_ListenSocket, AF_INET6); Result == SOCKET_ERROR)
	{
		throw std::system_error(std::error_code(WSAGetLastError(), std::system_category()),
								"listen call failed in HTTP plugin server init");
	}

	m_KeepRunning.test_and_set();

	m_AcceptThread = std::thread([&] {
		// SetCurrentThreadName("http_plugin_acceptor");

		// ZEN_INFO("HTTP plugin server waiting for connections");

		do
		{
			if (SOCKET ClientSocket = accept(m_ListenSocket, NULL, NULL); ClientSocket != SOCKET_ERROR)
			{
				int Flag = 1;
				setsockopt(ClientSocket, IPPROTO_TCP, TCP_NODELAY, (char*)&Flag, sizeof(Flag));

				// Handle new connection
				WinsockTransportConnection* Connection = new WinsockTransportConnection();
				TransportServerConnection*	ConnectionInterface{m_ServerInterface->CreateConnectionHandler(Connection)};
				Connection->Initialize(ConnectionInterface, ClientSocket);

				m_Connections.push_back(std::async(std::launch::async, [Connection] {
					try
					{
						Connection->HandleConnection();
					}
					catch (const std::exception&)
					{
						// ZEN_WARN("exception caught in connection loop: {}", Ex.what());
					}

					delete Connection;
				}));
			}
			else
			{
			}
		} while (m_KeepRunning.test());

		// ZEN_INFO("HTTP plugin server accept thread exit");
	});
}

void
WinsockTransportPlugin::Shutdown()
{
	// TODO: all pending/ongoing work should be drained here as well

	m_KeepRunning.clear();

	closesocket(m_ListenSocket);
	m_ListenSocket = 0;

	if (m_AcceptThread.joinable())
	{
		m_AcceptThread.join();
	}
}

const char*
WinsockTransportPlugin::GetDebugName()
{
	return nullptr;
}

bool
WinsockTransportPlugin::IsAvailable()
{
	return true;
}

//////////////////////////////////////////////////////////////////////////

void
GetTransportPluginVersion(uint32_t* OutApiVersion, uint32_t* OutPluginVersion)
{
	if (OutApiVersion != nullptr)
	{
		*OutApiVersion = kTransportApiVersion;
	}

	if (OutPluginVersion != nullptr)
	{
		*OutPluginVersion = 1;
	}
}

TransportPlugin*
CreateTransportPlugin([[maybe_unused]] TransportLogger* Logger)
{
	return new WinsockTransportPlugin;
}

BOOL WINAPI
DllMain([[maybe_unused]] HINSTANCE hinstDLL,   // handle to DLL module
		DWORD					   fdwReason,  // reason for calling function
		LPVOID					   lpvReserved)					   // reserved
{
	// Perform actions based on the reason for calling.
	switch (fdwReason)
	{
		case DLL_PROCESS_ATTACH:
			break;

		case DLL_THREAD_ATTACH:
			break;

		case DLL_THREAD_DETACH:
			break;

		case DLL_PROCESS_DETACH:
			if (lpvReserved != nullptr)
			{
				break;	// do not do cleanup if process termination scenario
			}
			break;
	}

	return TRUE;
}
