-- Copyright Epic Games, Inc. All Rights Reserved.

target("winsock")
    set_kind("shared")
    set_group("transports")
    add_headerfiles("**.h")
    add_files("**.cpp")
    add_links("Ws2_32")
    add_includedirs(".")
    set_symbols("debug")
    add_deps("transport-sdk", "zenbase")

    if is_mode("release") then
        set_optimize("fastest")
    end
