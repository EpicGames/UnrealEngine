-- Copyright Epic Games, Inc. All Rights Reserved.

set_warnings("allextra", "error")
set_languages("cxx20")

includes('transport-sdk')
includes('zenbase')

if is_plat("windows") then
    includes("winsock")
end
