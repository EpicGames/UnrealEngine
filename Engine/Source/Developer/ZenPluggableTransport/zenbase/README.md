This is a header-only library which mostly contains definitions to
support different platforms and compilers.

It is part of the zen codebase but is intended to be consumable
separately to zenbase etc to support standalone transport plug-ins 
and similar.
