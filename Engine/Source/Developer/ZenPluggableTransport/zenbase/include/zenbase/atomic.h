// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <zenbase/zenbase.h>

#if ZEN_COMPILER_MSC
#	include <intrin.h>
#else
#	include <atomic>
#endif

#include <cinttypes>

namespace zen {

inline uint32_t
AtomicIncrement(volatile uint32_t& value)
{
#if ZEN_COMPILER_MSC
	return _InterlockedIncrement((long volatile*)&value);
#else
	return ((std::atomic<uint32_t>*)(&value))->fetch_add(1, std::memory_order_seq_cst) + 1;
#endif
}
inline uint32_t
AtomicDecrement(volatile uint32_t& value)
{
#if ZEN_COMPILER_MSC
	return _InterlockedDecrement((long volatile*)&value);
#else
	return ((std::atomic<uint32_t>*)(&value))->fetch_sub(1, std::memory_order_seq_cst) - 1;
#endif
}

inline uint64_t
AtomicIncrement(volatile uint64_t& value)
{
#if ZEN_COMPILER_MSC
	return _InterlockedIncrement64((__int64 volatile*)&value);
#else
	return ((std::atomic<uint64_t>*)(&value))->fetch_add(1, std::memory_order_seq_cst) + 1;
#endif
}
inline uint64_t
AtomicDecrement(volatile uint64_t& value)
{
#if ZEN_COMPILER_MSC
	return _InterlockedDecrement64((__int64 volatile*)&value);
#else
	return ((std::atomic<uint64_t>*)(&value))->fetch_sub(1, std::memory_order_seq_cst) - 1;
#endif
}

inline uint32_t
AtomicAdd(volatile uint32_t& value, uint32_t amount)
{
#if ZEN_COMPILER_MSC
	return _InterlockedExchangeAdd((long volatile*)&value, amount);
#else
	return ((std::atomic<uint32_t>*)(&value))->fetch_add(amount, std::memory_order_seq_cst);
#endif
}
inline uint64_t
AtomicAdd(volatile uint64_t& value, uint64_t amount)
{
#if ZEN_COMPILER_MSC
	return _InterlockedExchangeAdd64((__int64 volatile*)&value, amount);
#else
	return ((std::atomic<uint64_t>*)(&value))->fetch_add(amount, std::memory_order_seq_cst);
#endif
}

}  // namespace zen
