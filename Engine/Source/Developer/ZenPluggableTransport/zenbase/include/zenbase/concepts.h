// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <zenbase/zenbase.h>

// At the time of writing only ver >= 13 of LLVM's libc++ has an implementation
// of std::integral. Some platforms like Ubuntu and Mac OS are still on 12.
#if defined(__cpp_lib_concepts)
#	include <concepts>
template<class T>
concept Integral = std::integral<T>;
template<class T>
concept SignedIntegral = std::signed_integral<T>;
template<class T>
concept UnsignedIntegral = std::unsigned_integral<T>;
template<class F, class... A>
concept Invocable = std::invocable<F, A...>;
template<class D, class B>
concept DerivedFrom = std::derived_from<D, B>;
#else
#	include <functional>  // for std::invoke below

template<class T>
concept Integral = std::is_integral_v<T>;
template<class T>
concept SignedIntegral = Integral<T> && std::is_signed_v<T>;
template<class T>
concept UnsignedIntegral = Integral<T> && !std::is_signed_v<T>;
template<class F, class... A>
concept Invocable = requires(F&& f, A&&... a)
{
	std::invoke(std::forward<F>(f), std::forward<A>(a)...);
};
template<class D, class B>
concept DerivedFrom = std::is_base_of_v<B, D> && std::is_convertible_v<const volatile D*, const volatile B*>;
#endif

#if defined(__cpp_lib_ranges)
#	include <ranges>
template<typename T>
concept ContiguousRange = std::ranges::contiguous_range<T>;
#else
template<typename T>
concept ContiguousRange = true;
#endif
