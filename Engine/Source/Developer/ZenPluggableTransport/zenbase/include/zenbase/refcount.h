// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <zenbase/atomic.h>
#include <zenbase/concepts.h>

#include <compare>

namespace zen {

/**
 * Helper base class for reference counted objects using intrusive reference counting
 */
class RefCounted
{
public:
	RefCounted()		  = default;
	virtual ~RefCounted() = default;

	inline uint32_t AddRef() const { return AtomicIncrement(const_cast<RefCounted*>(this)->m_RefCount); }
	inline uint32_t Release() const
	{
		const uint32_t RefCount = AtomicDecrement(const_cast<RefCounted*>(this)->m_RefCount);
		if (RefCount == 0)
		{
			delete this;
		}
		return RefCount;
	}

	// Copying reference counted objects doesn't make a lot of sense generally, so let's prevent it

	RefCounted(const RefCounted&) = delete;
	RefCounted(RefCounted&&)	  = delete;
	RefCounted& operator=(const RefCounted&) = delete;
	RefCounted& operator=(RefCounted&&) = delete;

protected:
	inline uint32_t RefCount() const { return m_RefCount; }

private:
	uint32_t m_RefCount = 0;
};

/**
 * Smart pointer for classes derived from RefCounted
 */

template<class T>
class RefPtr
{
public:
	inline RefPtr() = default;
	inline RefPtr(const RefPtr& Rhs) : m_Ref(Rhs.m_Ref) { m_Ref && m_Ref->AddRef(); }
	inline RefPtr(T* Ptr) : m_Ref(Ptr) { m_Ref && m_Ref->AddRef(); }
	inline ~RefPtr() { m_Ref && m_Ref->Release(); }

	[[nodiscard]] inline bool IsNull() const { return m_Ref == nullptr; }
	inline explicit			  operator bool() const { return m_Ref != nullptr; }
	inline					  operator T*() const { return m_Ref; }
	inline T*				  operator->() const { return m_Ref; }

	inline std::strong_ordering operator<=>(const RefPtr& Rhs) const = default;

	inline RefPtr& operator=(T* Rhs)
	{
		Rhs && Rhs->AddRef();
		m_Ref && m_Ref->Release();
		m_Ref = Rhs;
		return *this;
	}
	inline RefPtr& operator=(const RefPtr& Rhs)
	{
		if (&Rhs != this)
		{
			Rhs && Rhs->AddRef();
			m_Ref && m_Ref->Release();
			m_Ref = Rhs.m_Ref;
		}
		return *this;
	}
	inline RefPtr& operator=(RefPtr&& Rhs) noexcept
	{
		if (&Rhs != this)
		{
			m_Ref && m_Ref->Release();
			m_Ref	  = Rhs.m_Ref;
			Rhs.m_Ref = nullptr;
		}
		return *this;
	}
	template<typename OtherType>
	inline RefPtr& operator=(RefPtr<OtherType>&& Rhs) noexcept
	{
		if ((RefPtr*)&Rhs != this)
		{
			m_Ref && m_Ref->Release();
			m_Ref	  = Rhs.m_Ref;
			Rhs.m_Ref = nullptr;
		}
		return *this;
	}
	inline RefPtr(RefPtr&& Rhs) noexcept : m_Ref(Rhs.m_Ref) { Rhs.m_Ref = nullptr; }
	template<typename OtherType>
	explicit inline RefPtr(RefPtr<OtherType>&& Rhs) noexcept : m_Ref(Rhs.m_Ref)
	{
		Rhs.m_Ref = nullptr;
	}

private:
	T* m_Ref = nullptr;
	template<typename U>
	friend class RefPtr;
};

/**
 * Smart pointer for classes derived from RefCounted
 *
 * This variant does not decay to a raw pointer
 *
 */

template<class T>
class Ref
{
public:
	inline Ref() = default;
	inline Ref(const Ref& Rhs) : m_Ref(Rhs.m_Ref) { m_Ref && m_Ref->AddRef(); }
	inline explicit Ref(T* Ptr) : m_Ref(Ptr) { m_Ref && m_Ref->AddRef(); }
	inline ~Ref() { m_Ref && m_Ref->Release(); }

	template<typename DerivedType>
	requires DerivedFrom<DerivedType, T>
	inline Ref(const Ref<DerivedType>& Rhs) : Ref(Rhs.m_Ref) {}

	[[nodiscard]] inline bool IsNull() const { return m_Ref == nullptr; }
	inline explicit			  operator bool() const { return m_Ref != nullptr; }
	inline T*				  operator->() const { return m_Ref; }
	inline T&				  operator*() const { return *m_Ref; }
	inline T*				  Get() const { return m_Ref; }

	inline std::strong_ordering operator<=>(const Ref& Rhs) const = default;

	inline Ref& operator=(T* Rhs)
	{
		Rhs && Rhs->AddRef();
		m_Ref && m_Ref->Release();
		m_Ref = Rhs;
		return *this;
	}
	inline Ref& operator=(const Ref& Rhs)
	{
		if (&Rhs != this)
		{
			Rhs && Rhs->AddRef();
			m_Ref && m_Ref->Release();
			m_Ref = Rhs.m_Ref;
		}
		return *this;
	}
	inline Ref& operator=(Ref&& Rhs) noexcept
	{
		if (&Rhs != this)
		{
			m_Ref && m_Ref->Release();
			m_Ref	  = Rhs.m_Ref;
			Rhs.m_Ref = nullptr;
		}
		return *this;
	}
	inline Ref(Ref&& Rhs) noexcept : m_Ref(Rhs.m_Ref) { Rhs.m_Ref = nullptr; }

private:
	T* m_Ref = nullptr;

	template<class U>
	friend class Ref;
};

}  // namespace zen
