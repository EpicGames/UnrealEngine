// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cinttypes>
#include <version>

#ifdef __clang__
#	include <type_traits>
#endif

//////////////////////////////////////////////////////////////////////////
// Platform
//

#define ZEN_PLATFORM_WINDOWS 0
#define ZEN_PLATFORM_LINUX	 0
#define ZEN_PLATFORM_MAC	 0

#ifdef _WIN32
#	undef ZEN_PLATFORM_WINDOWS
#	define ZEN_PLATFORM_WINDOWS 1
#	define ZEN_PLATFORM_NAME	 "Windows"
#elif defined(__linux__)
#	undef ZEN_PLATFORM_LINUX
#	define ZEN_PLATFORM_LINUX 1
#	define ZEN_PLATFORM_NAME  "Linux"
#elif defined(__APPLE__)
#	undef ZEN_PLATFORM_MAC
#	define ZEN_PLATFORM_MAC  1
#	define ZEN_PLATFORM_NAME "MacOS"
#endif

#if ZEN_PLATFORM_WINDOWS
#	if !defined(NOMINMAX)
#		define NOMINMAX  // stops Windows.h from defining 'min/max' macros
#	endif
#	if !defined(NOGDI)
#		define NOGDI
#	endif
#	if !defined(WIN32_LEAN_AND_MEAN)
#		define WIN32_LEAN_AND_MEAN	 // cut-down what Windows.h defines
#	endif
#endif

//////////////////////////////////////////////////////////////////////////
// Compiler
//

#define ZEN_COMPILER_CLANG 0
#define ZEN_COMPILER_MSC   0
#define ZEN_COMPILER_GCC   0

// Clang can define __GNUC__ and/or _MSC_VER so we check for Clang first
#ifdef __clang__
#	undef ZEN_COMPILER_CLANG
#	define ZEN_COMPILER_CLANG 1
#elif defined(_MSC_VER)
#	undef ZEN_COMPILER_MSC
#	define ZEN_COMPILER_MSC 1
#elif defined(__GNUC__)
#	undef ZEN_COMPILER_GCC
#	define ZEN_COMPILER_GCC 1
#else
#	error Unknown compiler
#endif

#if ZEN_COMPILER_MSC
#	pragma warning(disable : 4324)	 // warning C4324: '<type>': structure was padded due to alignment specifier
#	pragma warning(default : 4668)	 // warning C4668: 'symbol' is not defined as a preprocessor macro, replacing with '0' for 'directives'
#	pragma warning(default : 4100)	 // warning C4100: 'identifier' : unreferenced formal parameter
#	pragma warning( \
		disable : 4373)	 // '%$S': virtual function overrides '%$pS', previous versions of the compiler did not override when parameters
						 // only differed by const/volatile qualifiers
						 // https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4373
#endif

#ifndef ZEN_THIRD_PARTY_INCLUDES_START
#	if ZEN_COMPILER_MSC
#		define ZEN_THIRD_PARTY_INCLUDES_START                                                                                      \
			__pragma(warning(push)) __pragma(warning(disable : 4668)) /* C4668: use of undefined preprocessor macro */              \
				__pragma(warning(disable : 4305))					  /* C4305: 'if': truncation from 'uint32' to 'bool' */         \
				__pragma(warning(disable : 4267))					  /* C4267: '=': conversion from 'size_t' to 'US' */            \
				__pragma(warning(disable : 4127))					  /* C4127: conditional expression is constant */               \
				__pragma(warning(disable : 4189))					  /* C4189: local variable is initialized but not referenced */ \
				__pragma(warning(disable : 5105)) /* C5105: macro expansion producing 'defined' has undefined behavior */
#	elif ZEN_COMPILER_CLANG
#		define ZEN_THIRD_PARTY_INCLUDES_START                                               \
			_Pragma("clang diagnostic push") _Pragma("clang diagnostic ignored \"-Wundef\"") \
				_Pragma("clang diagnostic ignored \"-Wunused-parameter\"") _Pragma("clang diagnostic ignored \"-Wunused-variable\"")
#	elif ZEN_COMPILER_GCC
#		define ZEN_THIRD_PARTY_INCLUDES_START                                              \
			_Pragma("GCC diagnostic push") /* NB. ignoring -Wundef doesn't work with GCC */ \
				_Pragma("GCC diagnostic ignored \"-Wunused-parameter\"") _Pragma("GCC diagnostic ignored \"-Wunused-variable\"")
#	endif
#endif

#ifndef ZEN_THIRD_PARTY_INCLUDES_END
#	if ZEN_COMPILER_MSC
#		define ZEN_THIRD_PARTY_INCLUDES_END __pragma(warning(pop))
#	elif ZEN_COMPILER_CLANG
#		define ZEN_THIRD_PARTY_INCLUDES_END _Pragma("clang diagnostic pop")
#	elif ZEN_COMPILER_GCC
#		define ZEN_THIRD_PARTY_INCLUDES_END _Pragma("GCC diagnostic pop")
#	endif
#endif

#if ZEN_COMPILER_MSC
#	define ZEN_DEBUG_BREAK() \
		do                    \
		{                     \
			__debugbreak();   \
		} while (0)
#else
#	define ZEN_DEBUG_BREAK() \
		do                    \
		{                     \
			__builtin_trap(); \
		} while (0)
#endif

//////////////////////////////////////////////////////////////////////////
// C++20 support
//

// Clang
#if ZEN_COMPILER_CLANG && __clang_major__ < 12
#	error clang-12 onwards is required for C++20 support
#endif

// GCC
#if ZEN_COMPILER_GCC && __GNUC__ < 11
#	error GCC-11 onwards is required for C++20 support
#endif

// GNU libstdc++
#if defined(_GLIBCXX_RELEASE) && _GLIBCXX_RELEASE < 11
#	error GNU libstdc++-11 onwards is required for C++20 support
#endif

// LLVM libc++
#if defined(_LIBCPP_VERSION) && _LIBCPP_VERSION < 12000
#	error LLVM libc++-12 onwards is required for C++20 support
#endif

//////////////////////////////////////////////////////////////////////////
// Architecture
//

#if defined(__amd64__) || defined(_M_X64)
#	define ZEN_ARCH_X64   1
#	define ZEN_ARCH_ARM64 0
#elif defined(__arm64__) || defined(_M_ARM64)
#	define ZEN_ARCH_X64   0
#	define ZEN_ARCH_ARM64 1
#else
#	error Unknown architecture
#endif

//////////////////////////////////////////////////////////////////////////
// Build flavor
//

#ifdef NDEBUG
#	define ZEN_BUILD_DEBUG	  0
#	define ZEN_BUILD_RELEASE 1
#	define ZEN_BUILD_NAME	  "release"
#else
#	define ZEN_BUILD_DEBUG	  1
#	define ZEN_BUILD_RELEASE 0
#	define ZEN_BUILD_NAME	  "debug"
#endif

//////////////////////////////////////////////////////////////////////////

#define ZEN_PLATFORM_SUPPORTS_UNALIGNED_LOADS 1

#if defined(__SIZEOF_WCHAR_T__) && __SIZEOF_WCHAR_T__ == 4
#	define ZEN_SIZEOF_WCHAR_T 4
#else
static_assert(sizeof(wchar_t) == 2, "wchar_t is expected to be two bytes in size");
#	define ZEN_SIZEOF_WCHAR_T 2
#endif

//////////////////////////////////////////////////////////////////////////

#ifdef __clang__
template<typename T>
auto ZenArrayCountHelper(T& t) -> typename std::enable_if<__is_array(T), char (&)[sizeof(t) / sizeof(t[0]) + 1]>::type;
#else
template<typename T, uint32_t N>
char (&ZenArrayCountHelper(const T (&)[N]))[N + 1];
#endif

#define ZEN_ARRAY_COUNT(array) (sizeof(ZenArrayCountHelper(array)) - 1)

//////////////////////////////////////////////////////////////////////////

#if ZEN_COMPILER_MSC
#	define ZEN_NOINLINE	__declspec(noinline)
#	define ZEN_FORCEINLINE [[msvc::forceinline]]
#else
#	define ZEN_NOINLINE	__attribute__((noinline))
#	define ZEN_FORCEINLINE __attribute__((always_inline))
#endif

#if ZEN_PLATFORM_WINDOWS
#	define ZEN_EXE_SUFFIX_LITERAL ".exe"
#else
#	define ZEN_EXE_SUFFIX_LITERAL ""
#endif

#define ZEN_UNUSED(...) ((void)__VA_ARGS__)

//////////////////////////////////////////////////////////////////////////

#if ZEN_COMPILER_MSC
#	define ZEN_DISABLE_OPTIMIZATION_ACTUAL __pragma(optimize("", off))
#	define ZEN_ENABLE_OPTIMIZATION_ACTUAL	__pragma(optimize("", on))
#elif ZEN_COMPILER_GCC
#	define ZEN_DISABLE_OPTIMIZATION_ACTUAL _Pragma("GCC push_options") _Pragma("GCC optimize (\"O0\")")
#	define ZEN_ENABLE_OPTIMIZATION_ACTUAL	_Pragma("GCC pop_options")
#elif ZEN_COMPILER_CLANG
#	define ZEN_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
#	define ZEN_ENABLE_OPTIMIZATION_ACTUAL	_Pragma("clang optimize on")
#endif

// Set up optimization control macros, now that we have both the build settings and the platform macros
#define ZEN_DISABLE_OPTIMIZATION ZEN_DISABLE_OPTIMIZATION_ACTUAL

#if ZEN_BUILD_DEBUG
#	define ZEN_ENABLE_OPTIMIZATION ZEN_DISABLE_OPTIMIZATION_ACTUAL
#else
#	define ZEN_ENABLE_OPTIMIZATION ZEN_ENABLE_OPTIMIZATION_ACTUAL
#endif

#define ZEN_ENABLE_OPTIMIZATION_ALWAYS ZEN_ENABLE_OPTIMIZATION_ACTUAL

#if ZEN_PLATFORM_WINDOWS
// Tells the compiler to put the decorated function in a certain section (aka. segment) of the executable.
#	define ZEN_CODE_SECTION(Name) __declspec(code_seg(Name))
#	define ZEN_DATA_SECTION(Name) __declspec(allocate(Name))
#	define ZEN_FORCENOINLINE	   __declspec(noinline) /* Force code to NOT be inline */
#	define LINE_TERMINATOR_ANSI   "\r\n"
#else
#	define ZEN_CODE_SECTION(Name)
#	define ZEN_DATA_SECTION(Name)
#	define ZEN_FORCENOINLINE
#	define LINE_TERMINATOR_ANSI "\n"
#endif
