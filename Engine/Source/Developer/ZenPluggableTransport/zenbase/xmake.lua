-- Copyright Epic Games, Inc. All Rights Reserved.

target('zenbase')
    set_kind("headeronly")
    set_group("libs")
    add_headerfiles("**.h")
    add_includedirs("include", {public=true})
